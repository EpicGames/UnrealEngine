---
AccessModifierOffset: -4
AlignAfterOpenBracket: DontAlign
AlignArrayOfStructures: Right
AlignConsecutiveAssignments: None
AlignConsecutiveBitFields: None
AlignConsecutiveDeclarations: None
AlignEscapedNewlines: Left
AlignOperands: AlignAfterOperator
AlignTrailingComments: true
AllowShortBlocksOnASingleLine: Empty
AllowShortEnumsOnASingleLine: false
AllowShortFunctionsOnASingleLine: Inline
AllowShortLambdasOnASingleLine: Inline
BasedOnStyle: LLVM
BraceWrapping:
  AfterCaseLabel: true
  AfterClass: true
  AfterControlStatement: true
  AfterEnum: true
  AfterFunction: true
  AfterNamespace: true
  AfterObjCDeclaration: true
  AfterStruct: true
  AfterUnion: true
  AfterExternBlock: true
  BeforeCatch: true
  BeforeElse: true
  BeforeLambdaBody: false
  BeforeWhile: true
  IndentBraces: false
BreakBeforeBinaryOperators: NonAssignment
BreakBeforeBraces: Custom
BreakBeforeTernaryOperators: true
BreakConstructorInitializers: BeforeComma
BreakInheritanceList: BeforeComma
BreakStringLiterals: true
ColumnLimit: 0
Cpp11BracedListStyle: true
EmptyLineBeforeAccessModifier: LogicalBlock
FixNamespaceComments: true
IncludeBlocks: Preserve
IndentCaseBlocks: false
IndentCaseLabels: true
IndentPPDirectives: None
IndentWidth: 4
Language: Cpp
NamespaceIndentation: None
PackConstructorInitializers: CurrentLine
PointerAlignment: Left
SortIncludes: true
SpaceBeforeCaseColon: false
Standard: Cpp11
StatementMacros: ['UPROPERTY', 'UFUNCTION', 'UCLASS', 'USTRUCT', 'UENUM', 'UINTERFACE', 'GENERATED_BODY', 'v']
TabWidth: 4
UseTab: ForContinuationAndIndentation
