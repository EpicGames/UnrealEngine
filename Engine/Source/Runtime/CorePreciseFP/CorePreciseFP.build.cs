// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class CorePreciseFP : ModuleRules
{
	public CorePreciseFP(ReadOnlyTargetRules Target) : base(Target)
	{
		FPSemantics = FPSemanticsMode.Precise;
		PCHUsage = PCHUsageMode.NoPCHs;

		PublicDependencyModuleNames.AddRange(
			new string[]
			{
				"Core",
			}
		);
	}
}
