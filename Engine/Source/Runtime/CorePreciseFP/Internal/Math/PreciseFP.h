// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"

namespace UE
{
COREPRECISEFP_API bool PreciseFPEqual(float Lhs, float Rhs);
COREPRECISEFP_API bool PreciseFPEqual(double Lhs, double Rhs);

COREPRECISEFP_API uint32 PreciseFPHash(float);
COREPRECISEFP_API uint32 PreciseFPHash(double);
} // namespace UE