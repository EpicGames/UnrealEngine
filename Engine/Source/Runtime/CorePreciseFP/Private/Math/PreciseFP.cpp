// Copyright Epic Games, Inc. All Rights Reserved.

#include "Math/PreciseFP.h"
#include "Templates/TypeCompatibleBytes.h"
#include "Templates/TypeHash.h"
#include <cmath>

namespace UE
{
bool PreciseFPEqual(float Lhs, float Rhs)
{
	// NaNs compare as equal to each other, and unequal to numbers.
	if (std::isnan(Lhs))
	{
		return std::isnan(Rhs);
	}
	return Lhs == Rhs;
}

bool PreciseFPEqual(double Lhs, double Rhs)
{
	// NaNs compare as equal to each other, and unequal to numbers.
	if (std::isnan(Lhs))
	{
		return std::isnan(Rhs);
	}
	return Lhs == Rhs;
}

uint32 PreciseFPHash(float F)
{
	// Treat all NaNs and zeroes as equal.
	if (std::isnan(F) || F == 0.0f)
	{
		return 0;
	}

	return ::GetTypeHash(BitCast<uint32>(F));
}

uint32 PreciseFPHash(double D)
{
	// Treat all NaNs and zeroes as equal.
	if (std::isnan(D) || D == 0.0)
	{
		return 0;
	}

	return ::GetTypeHash(BitCast<uint64>(D));
}
} // namespace UE
